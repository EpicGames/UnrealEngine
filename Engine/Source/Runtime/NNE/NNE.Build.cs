// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NNE : ModuleRules
	{
		public NNE(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject"
				}
			);

			if (Target.Type == TargetType.Editor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"DerivedDataCache"
					}
				);

				DynamicallyLoadedModuleNames.Add("NNEEditor");
			}
		}
	}
}
