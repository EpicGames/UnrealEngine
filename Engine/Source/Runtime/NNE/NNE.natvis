<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <!-- 
  *
  * NNE Visualizers 
  *
  -->

  <!-- NNE::Core visualizer -->
  <Type Name="UE::NNE::Internal::FTensor">
    <DisplayString Condition="PreparedData.ArrayNum == 0">Name={Name,sb} Shape={Shape}</DisplayString>
    <DisplayString Condition="PreparedData.ArrayNum &lt;= 4*8">Name={Name,sb} Shape={Shape} Data={(float*)PreparedData.AllocatorInstance.Data,[PreparedData.ArrayNum/4]}</DisplayString>
    <DisplayString Condition="PreparedData.ArrayNum &gt; 4*8">Name={Name,sb} Shape={Shape} Data={(float*)PreparedData.AllocatorInstance.Data,8}</DisplayString>
  </Type>

  <Type Name="UE::NNE::FTensorDesc">
    <DisplayString>Name={Name,sb} Shape={Shape}</DisplayString>
  </Type>

  <Type Name="UE::NNE::FTensorShape">
    <DisplayString Condition="Data.ArrayNum == 0">
      Scalar
    </DisplayString>
    <DisplayString Condition="Data.ArrayNum == 1">
      [{Data.AllocatorInstance.InlineData[0]}]
    </DisplayString>
    <DisplayString Condition="Data.ArrayNum == 2">
      [{Data.AllocatorInstance.InlineData[0]},{Data.AllocatorInstance.InlineData[1]}]
     </DisplayString>
    <DisplayString Condition="Data.ArrayNum == 3">
      [{Data.AllocatorInstance.InlineData[0]},{Data.AllocatorInstance.InlineData[1]},{Data.AllocatorInstance.InlineData[2]}]
     </DisplayString>
    <DisplayString Condition="Data.ArrayNum == 4">
      [{Data.AllocatorInstance.InlineData[0]},{Data.AllocatorInstance.InlineData[1]},{Data.AllocatorInstance.InlineData[2]},{Data.AllocatorInstance.InlineData[3]}]
    </DisplayString>
    <DisplayString Condition="Data.ArrayNum == 5">
      [{Data.AllocatorInstance.InlineData[0]},{Data.AllocatorInstance.InlineData[1]},{Data.AllocatorInstance.InlineData[2]},{Data.AllocatorInstance.InlineData[3]},{Data.AllocatorInstance.InlineData[4]}]
    </DisplayString>
    <DisplayString Condition="Data.ArrayNum == 6">
      [{Data.AllocatorInstance.InlineData[0]},{Data.AllocatorInstance.InlineData[1]},{Data.AllocatorInstance.InlineData[2]},{Data.AllocatorInstance.InlineData[3]},{Data.AllocatorInstance.InlineData[4]},{Data.AllocatorInstance.InlineData[5]}]
    </DisplayString>
    <DisplayString Condition="Data.ArrayNum == 7">
      [{Data.AllocatorInstance.InlineData[0]},{Data.AllocatorInstance.InlineData[1]},{Data.AllocatorInstance.InlineData[2]},{Data.AllocatorInstance.InlineData[3]},{Data.AllocatorInstance.InlineData[4]},{Data.AllocatorInstance.InlineData[5]},{Data.AllocatorInstance.InlineData[6]}]
    </DisplayString>
    <DisplayString Condition="Data.ArrayNum == 8">
      [{Data.AllocatorInstance.InlineData[0]},{Data.AllocatorInstance.InlineData[1]},{Data.AllocatorInstance.InlineData[2]},{Data.AllocatorInstance.InlineData[3]},{Data.AllocatorInstance.InlineData[4]},{Data.AllocatorInstance.InlineData[5]},{Data.AllocatorInstance.InlineData[7]}]    </DisplayString>
    <DisplayString Condition="Data.ArrayNum &gt;= 9">
      {Data.AllocatorInstance.InlineData}
    </DisplayString>
  </Type>

  <Type Name="UE::NNE::FSymbolicTensorShape">
    <DisplayString Condition="Data.ArrayNum == 0">
      Scalar
    </DisplayString>
    <DisplayString Condition="Data.ArrayNum == 1">
      [{Data.AllocatorInstance.InlineData[0]}]
    </DisplayString>
    <DisplayString Condition="Data.ArrayNum == 2">
      [{Data.AllocatorInstance.InlineData[0]},{Data.AllocatorInstance.InlineData[1]}]
    </DisplayString>
    <DisplayString Condition="Data.ArrayNum == 3">
      [{Data.AllocatorInstance.InlineData[0]},{Data.AllocatorInstance.InlineData[1]},{Data.AllocatorInstance.InlineData[2]}]
    </DisplayString>
    <DisplayString Condition="Data.ArrayNum == 4">
      [{Data.AllocatorInstance.InlineData[0]},{Data.AllocatorInstance.InlineData[1]},{Data.AllocatorInstance.InlineData[2]},{Data.AllocatorInstance.InlineData[3]}]
    </DisplayString>
    <DisplayString Condition="Data.ArrayNum == 5">
      [{Data.AllocatorInstance.InlineData[0]},{Data.AllocatorInstance.InlineData[1]},{Data.AllocatorInstance.InlineData[2]},{Data.AllocatorInstance.InlineData[3]},{Data.AllocatorInstance.InlineData[4]}]
    </DisplayString>
    <DisplayString Condition="Data.ArrayNum == 6">
      [{Data.AllocatorInstance.InlineData[0]},{Data.AllocatorInstance.InlineData[1]},{Data.AllocatorInstance.InlineData[2]},{Data.AllocatorInstance.InlineData[3]},{Data.AllocatorInstance.InlineData[4]},{Data.AllocatorInstance.InlineData[5]}]
    </DisplayString>
    <DisplayString Condition="Data.ArrayNum == 7">
      [{Data.AllocatorInstance.InlineData[0]},{Data.AllocatorInstance.InlineData[1]},{Data.AllocatorInstance.InlineData[2]},{Data.AllocatorInstance.InlineData[3]},{Data.AllocatorInstance.InlineData[4]},{Data.AllocatorInstance.InlineData[5]},{Data.AllocatorInstance.InlineData[6]}]
    </DisplayString>
    <DisplayString Condition="Data.ArrayNum == 8">
      [{Data.AllocatorInstance.InlineData[0]},{Data.AllocatorInstance.InlineData[1]},{Data.AllocatorInstance.InlineData[2]},{Data.AllocatorInstance.InlineData[3]},{Data.AllocatorInstance.InlineData[4]},{Data.AllocatorInstance.InlineData[5]},{Data.AllocatorInstance.InlineData[7]}]
    </DisplayString>
    <DisplayString Condition="Data.ArrayNum &gt;= 9">
{Data.AllocatorInstance.InlineData}
    </DisplayString>
  </Type>
  
</AutoVisualizer>
