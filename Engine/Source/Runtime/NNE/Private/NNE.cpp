// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNE.h"

#include "Misc/CoreDelegates.h"
#include "Misc/SecureHash.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogNNE);

namespace UE::NNE
{

	class FRegistry
	{
	public:
		static FRegistry& GetInstance()
		{
			static FRegistry Instance;
			return Instance;
		}

		EResultStatus Add(TWeakInterfacePtr<INNERuntime> Runtime)
		{
			checkf(Runtime.IsValid(), TEXT("Runtime is not valid"));

			const FString RuntimeName = Runtime->GetRuntimeName();
			checkf(!RuntimeName.IsEmpty(), TEXT("Runtime name is empty"));

			if (Runtimes.Contains(RuntimeName))
			{
				UE_LOGF(LogNNE, Warning, "Runtime %ls is already registered", *RuntimeName);
				return EResultStatus::Fail;
			}

			Runtimes.Add(RuntimeName, Runtime);

			return EResultStatus::Ok;
		}

		EResultStatus Remove(TWeakInterfacePtr<INNERuntime> Runtime)
		{
			checkf(Runtime.IsValid(), TEXT("Runtime is not valid"));

			const FString RuntimeName = Runtime->GetRuntimeName();
			checkf(!RuntimeName.IsEmpty(), TEXT("Runtime name is empty"));

			return Runtimes.Remove(RuntimeName) >= 1 ? EResultStatus::Ok : EResultStatus::Fail;
		}

		TWeakInterfacePtr<INNERuntime> Get(const FString& Name) const
		{
			checkf(!Name.IsEmpty(), TEXT("Name is empty"));

			if (Runtimes.Contains(Name))
			{
				TWeakInterfacePtr<INNERuntime> Result = Runtimes.FindChecked(Name);
				ensureMsgf(Result.IsValid(), TEXT("Runtime %s is not valid"), *Name);

				return Result;
			}

			return {};
		}

		TArray<FString> GetAllNames() const
		{
			TArray<FString> Result;
			Runtimes.GenerateKeyArray(Result);

			Result.SetNum(Algo::RemoveIf(Result, [&] (const FString &RuntimeName)
			{
				return !ensureMsgf(Runtimes.FindChecked(RuntimeName).IsValid(), TEXT("Runtime %s is not valid"), *RuntimeName);
			}));

			return Result;
		}

	private:
		TMap<FString, TWeakInterfacePtr<INNERuntime>> Runtimes;
	};

	ERegisterRuntimeStatus RegisterRuntime(TWeakInterfacePtr<INNERuntime> Runtime)
	{
#ifdef WITH_EDITOR
		FModuleManager::Get().LoadModule(TEXT("NNEEditor"));
#endif

		return FRegistry::GetInstance().Add(Runtime);
	}

	EUnregisterRuntimeStatus UnregisterRuntime(TWeakInterfacePtr<INNERuntime> Runtime)
	{
		return FRegistry::GetInstance().Remove(Runtime);
	}

	TArray<FString> GetAllRuntimeNames()
	{
		return FRegistry::GetInstance().GetAllNames();
	}

	TWeakInterfacePtr<INNERuntime> GetRuntime(const FString& Name)
	{
		return FRegistry::GetInstance().Get(Name);
	}
}
