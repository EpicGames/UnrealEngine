// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEModelData.h"

#include "NNE.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/WeakInterfacePtr.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NNEModelData)

namespace UE::NNE::ModelData
{
	enum Version : uint32
	{
		V0 = 0, // Initial
		V1 = 1, // TargetRuntimes and AssetImportData
		V2 = 2, // Re-arrange fields and store only ModelData in cooked assets
		V3 = 3, // Adding AdditionalFileData
		V4 = 4, // Support for > 2GB models
		V5 = 5, // Adding RuntimeSettings
		// New versions can be added above this line
		VersionPlusOne,
		Latest = VersionPlusOne - 1
	};

	const FGuid GUID(0x9513202e, 0xeba1b279, 0xf17fe5ba, 0xab90c3f2);
	FCustomVersionRegistration NNEModelDataVersion(GUID, Version::Latest, TEXT("NNEModelDataVersion"));// Always save with the latest version

	FString GetRuntimesAsString(TArrayView<const FString> Runtimes)
	{
		if (Runtimes.Num() == 0)
		{
			return TEXT("All");
		}

		FString RuntimesAsOneString;
		bool bIsFirstRuntime = true;

		for (const FString& Runtime : Runtimes)
		{
			if (!bIsFirstRuntime)
			{
				RuntimesAsOneString += TEXT(", ");
			}
			RuntimesAsOneString += Runtime;
			bIsFirstRuntime = false;
		}
		return RuntimesAsOneString;
	}

	enum class ESerializationStatus : uint8
	{
		Success = 0,
		InvalidCustomVersion = 1,
		NewerCustomVersion = 2,
		SerializationError = 3,
	};

	ESerializationStatus DeserializeRuntimeSettings(TConstArrayView64<uint8> InSerializedData, UObject* ModelSetting)
	{
		FMemoryReaderView Reader(InSerializedData, /*bIsPersistent*/ true);
		FCustomVersionContainer Versions;
		TArray64<uint8> ObjectData;
		Versions.Serialize(Reader, ECustomVersionSerializationFormat::Optimized);
		Reader << ObjectData;
		if (Reader.IsError() || !Reader.AtEnd())
		{
			return ESerializationStatus::SerializationError;
		}
		TArray<FCustomVersionDifference> Diffs = FCurrentCustomVersions::Compare(Versions.GetAllVersions(), *ModelSetting->GetFName().ToString());
		for (FCustomVersionDifference Diff : Diffs)
		{
			if (Diff.Type == ECustomVersionDifference::Invalid)
			{
				return ESerializationStatus::InvalidCustomVersion;
			}
			else if (Diff.Type == ECustomVersionDifference::Newer)
			{
				return ESerializationStatus::NewerCustomVersion;
			}
		}
		FMemoryReaderView ObjectReader(ObjectData, /*bIsPersistent*/ true);
		ObjectReader.SetCustomVersions(Versions);
		ModelSetting->Serialize(ObjectReader);
		return ObjectReader.IsError() || !ObjectReader.AtEnd() ? ESerializationStatus::SerializationError : ESerializationStatus::Success;
	}

	ESerializationStatus SerializeRuntimeSettings(UObject* ModelSetting, TArray64<uint8>& OutSerializedData)
	{
		TArray64<uint8> ObjectData;
		FMemoryWriter64 ObjectWriter(ObjectData, /*bIsPersistent*/ true);
		ModelSetting->Serialize(ObjectWriter);
		if (ObjectWriter.IsError())
		{
			return ESerializationStatus::SerializationError;
		}
		FCustomVersionContainer Versions = ObjectWriter.GetCustomVersions();
		FMemoryWriter64 Writer(OutSerializedData, /*bIsPersistent*/ true);
		Versions.Serialize(Writer, ECustomVersionSerializationFormat::Optimized);
		Writer << ObjectData;
		return Writer.IsError() ? ESerializationStatus::SerializationError : ESerializationStatus::Success;
	}

	template<class Value> 
	const FString* FindLongestKeyWithPrefix(const TMap<FString, Value>& Map, const FString& Prefix)
	{
		const FString* Result = nullptr;
		int32 LongestLength = 0;
		for (const auto& Element : Map)
		{
			const FString& Key = Element.Key;
			if (Key.Len() > LongestLength && Key.StartsWith(Prefix, ESearchCase::CaseSensitive))
			{
				LongestLength = Key.Len();
				Result = &Key;
			}
		}
		return Result;
	}
} // UE::NNE::ModelData

namespace UE::NNE
{
	FSharedModelData::FSharedModelData(FSharedBuffer InData, uint32 InMemoryAlignment) : Data(InData), MemoryAlignment(InMemoryAlignment)
	{
		checkf(Data.IsOwned(), TEXT("InData data must be ownned!"));
		checkf(MemoryAlignment <= 1 || (((uintptr_t)(const void*)(InData.GetData())) % MemoryAlignment == 0), TEXT("InData must be aligned with InMemoryAlignment!"))
	}

	FSharedModelData::FSharedModelData()
	{
		MemoryAlignment = 0;
	}

	TConstArrayView64<uint8> FSharedModelData::GetView() const
	{
		return TConstArrayView64<uint8>(static_cast<const uint8*>(Data.GetData()), Data.GetSize());
	}

	uint32 FSharedModelData::GetMemoryAlignment() const
	{
		return MemoryAlignment;
	}
} // UE::NNE

void UNNEModelData::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Context.AddTag(FAssetRegistryTag("TargetRuntimes", UE::NNE::ModelData::GetRuntimesAsString(GetTargetRuntimes()), FAssetRegistryTag::TT_Alphabetical));
	Super::GetAssetRegistryTags(Context);
}

void UNNEModelData::Serialize(FArchive& Ar)
{
	// Store the asset version (no effect in load)
	Ar.UsingCustomVersion(UE::NNE::ModelData::GUID);

	if (Ar.IsSaving() || Ar.IsCountingMemory())	{
		bool bWriteModelData = true;
		if (Ar.IsCooking())
		{
			// Optimize storage: FileData, AdditionalFileData and RuntimeSettings are not required anymore because we have the model and can cook it for every runtime
			TArray<FString> TmpTargetRuntimes;
			Ar << TmpTargetRuntimes;
			FString TmpFileType;
			Ar << TmpFileType;
			TArray64<uint8> TmpFileData;
			Ar << TmpFileData;
			int32 NumAdditionalFileDataItems = 0;
			Ar << NumAdditionalFileDataItems;
			int32 NumRuntimeSettings = 0;
			Ar << NumRuntimeSettings;
			FGuid TmpGuid;
			Ar << TmpGuid;

			// Cooking must recreate all model data but only if file data is still available
			if (FileData.Num() > 0)
			{
				ModelData.Reset();

				// No target runtime means all currently registered ones
				TArray<FString, TInlineAllocator<10>> CookRuntimeNames;
				if (GetTargetRuntimes().IsEmpty())
				{
					CookRuntimeNames.Append(UE::NNE::GetAllRuntimeNames());
				}
				else
				{
					CookRuntimeNames.Append(GetTargetRuntimes());
				}

				for (const FString& RuntimeName : CookRuntimeNames)
				{
					TSharedPtr<UE::NNE::FSharedModelData> SharedModelData = CreateModelData(RuntimeName, Ar.GetArchiveState().CookingTarget());
					if (SharedModelData.IsValid() && SharedModelData->GetView().Num() > 0)
					{
						ModelData.Add(RuntimeName, SharedModelData);
					}
				}
			}
		}
		else
		{
			// Only cooked assets optimize storage
			Ar << TargetRuntimes;
			Ar << FileType;
			Ar << FileData;

			int32 NumAdditionalFileDataItems = AdditionalFileData.Num();
			Ar << NumAdditionalFileDataItems;
			for (auto& Element : AdditionalFileData)
			{
				Ar << Element.Key;
				Ar << Element.Value;
			}

			int32 NumRuntimeSettings = RuntimeSettings.Num();
			Ar << NumRuntimeSettings;
			for (auto& Element : RuntimeSettings)
			{
				Ar << Element.Key;
				Ar << Element.Value;
			}

			Ar << FileId;

#if WITH_EDITOR
			// In editor (when not cooking), no model data is stored as model data can always be recreated and unnecessary data in subversion control should be avoided
			bWriteModelData = false;
#endif //WITH_EDITOR
		}

		if (bWriteModelData)
		{
			TArray<FString> RuntimeNames;
			ModelData.GetKeys(RuntimeNames);
			int32 NumItems = RuntimeNames.Num();

			Ar << NumItems;
			for (int32 i = 0; i < NumItems; i++)
			{
				Ar << RuntimeNames[i];

				uint32 MemoryAlignment = ModelData[RuntimeNames[i]]->GetMemoryAlignment();
				Ar << MemoryAlignment;

				uint64 DataSize = ModelData[RuntimeNames[i]]->GetView().Num();
				Ar << DataSize;

				Ar.Serialize((void*)ModelData[RuntimeNames[i]]->GetView().GetData(), DataSize);
			}
		}
		else
		{
			int32 NumItems = 0;
			Ar << NumItems;
		}
	}
	else if (Ar.IsLoading())
	{
		TObjectPtr<class UAssetImportData> AssetImportData;
		int32 NumItems = 0;
		int32 NumAdditionalFileDataItems = 0;
		int32 NumRuntimeSettings = 0;
		FString Name;
		uint32 MemoryAlignment = 0;
		uint64 DataSize = 0;
		TArray64<uint8> Data;
		TArray<uint8> Data_Size32Bits;
		void* RawData = nullptr;
		int32 Index = 0;

		const int32 Version = Ar.CustomVer(UE::NNE::ModelData::GUID);
		switch (Version)
		{
		case UE::NNE::ModelData::Version::V0:
			TargetRuntimes.Empty();
			Ar << FileType;
			Ar << Data_Size32Bits;
			FileData = MoveTemp(Data_Size32Bits);
			Ar << FileId;
			Ar << NumItems;
			for (Index = 0; Index < NumItems; Index++)
			{
				Ar << Name;
				Ar << Data_Size32Bits;
				ModelData.Add(Name, MakeShared<UE::NNE::FSharedModelData>(MakeSharedBufferFromArray(MoveTemp(Data_Size32Bits)), 0));
			}
			UE_LOGF(LogNNE, Warning, "[DEPRECATION] The asset %ls (v0) is deprecated. Please right-click the asset and select 'Save' to update it to the latest version.", *this->GetName());
			break;

		case UE::NNE::ModelData::Version::V1:
			TargetRuntimes.Empty();
			if (!Ar.IsLoadingFromCookedPackage())
			{
				Ar << TargetRuntimes;
				Ar << AssetImportData;
			}
			Ar << FileType;
			Ar << Data_Size32Bits;
			FileData = MoveTemp(Data_Size32Bits);
			Ar << FileId;
			Ar << NumItems;
			for (Index = 0; Index < NumItems; Index++)
			{
				Ar << Name;
				Ar << Data_Size32Bits;
				ModelData.Add(Name, MakeShared<UE::NNE::FSharedModelData>(MakeSharedBufferFromArray(MoveTemp(Data_Size32Bits)), 0));
			}
			UE_LOGF(LogNNE, Warning, "[DEPRECATION] The asset %ls (v1) is deprecated. Please right-click the asset and select 'Save' to update it to the latest version.", *this->GetName());
			break;

		case UE::NNE::ModelData::Version::V2:
			Ar << TargetRuntimes;
			Ar << FileType;
			Ar << Data_Size32Bits;
			FileData = MoveTemp(Data_Size32Bits);
			Ar << FileId;
			Ar << NumItems;
			for (Index = 0; Index < NumItems; Index++)
			{
				Ar << Name;
				Ar << MemoryAlignment;
				Ar << DataSize;
				RawData = FMemory::Malloc(DataSize, MemoryAlignment);
				Ar.Serialize(RawData, DataSize);
				ModelData.Add(Name, MakeShared<UE::NNE::FSharedModelData>(FSharedBuffer::TakeOwnership(RawData, DataSize, FMemory::Free), MemoryAlignment));
			}
			break;

		case UE::NNE::ModelData::Version::V3:
			Ar << TargetRuntimes;
			Ar << FileType;
			Ar << Data_Size32Bits;
			FileData = MoveTemp(Data_Size32Bits);
			Ar << NumAdditionalFileDataItems;
			AdditionalFileData.Empty();
			for (Index = 0; Index < NumAdditionalFileDataItems; Index++)
			{
				Ar << Name;
				Ar << Data_Size32Bits;
				Data = MoveTemp(Data_Size32Bits);
				AdditionalFileData.Add(Name, Data);
			}
			Ar << FileId;
			Ar << NumItems;
			for (Index = 0; Index < NumItems; Index++)
			{
				Ar << Name;
				Ar << MemoryAlignment;
				Ar << DataSize;
				RawData = FMemory::Malloc(DataSize, MemoryAlignment);
				Ar.Serialize(RawData, DataSize);
				ModelData.Add(Name, MakeShared<UE::NNE::FSharedModelData>(FSharedBuffer::TakeOwnership(RawData, DataSize, FMemory::Free), MemoryAlignment));
			}
			break;

		case UE::NNE::ModelData::Version::V4:
			Ar << TargetRuntimes;
			Ar << FileType;
			Ar << FileData;
			Ar << NumAdditionalFileDataItems;
			AdditionalFileData.Empty();
			for (Index = 0; Index < NumAdditionalFileDataItems; Index++)
			{
				Ar << Name;
				Ar << Data;
				AdditionalFileData.Add(Name, Data);
			}
			Ar << FileId;
			Ar << NumItems;
			for (Index = 0; Index < NumItems; Index++)
			{
				Ar << Name;
				Ar << MemoryAlignment;
				Ar << DataSize;
				RawData = FMemory::Malloc(DataSize, MemoryAlignment);
				Ar.Serialize(RawData, DataSize);
				ModelData.Add(Name, MakeShared<UE::NNE::FSharedModelData>(FSharedBuffer::TakeOwnership(RawData, DataSize, FMemory::Free), MemoryAlignment));
			}
			break;

		case UE::NNE::ModelData::Version::V5:
			Ar << TargetRuntimes;
			Ar << FileType;
			Ar << FileData;
			Ar << NumAdditionalFileDataItems;
			AdditionalFileData.Empty();
			for (Index = 0; Index < NumAdditionalFileDataItems; Index++)
			{
				Ar << Name;
				Ar << Data;
				AdditionalFileData.Add(Name, Data);
			}
			Ar << NumRuntimeSettings;
			RuntimeSettings.Reset();
			for (Index = 0; Index < NumRuntimeSettings; Index++)
			{
				Ar << Name;
				Ar << Data;
				RuntimeSettings.Add(Name, Data);
			}
			Ar << FileId;
			Ar << NumItems;
			for (Index = 0; Index < NumItems; Index++)
			{
				Ar << Name;
				Ar << MemoryAlignment;
				Ar << DataSize;
				RawData = FMemory::Malloc(DataSize, MemoryAlignment);
				Ar.Serialize(RawData, DataSize);
				ModelData.Add(Name, MakeShared<UE::NNE::FSharedModelData>(FSharedBuffer::TakeOwnership(RawData, DataSize, FMemory::Free), MemoryAlignment));
			}
			break;

		default:
			UE_LOGF(LogNNE, Error, "Unknown asset version %d: Deserialisation failed, please reimport the original model.", Ar.CustomVer(UE::NNE::ModelData::GUID));
			break;
		}
	}
}

void UNNEModelData::Init(const FString& Type, TConstArrayView64<uint8> Buffer, const TMap<FString, TConstArrayView64<uint8>>& AdditionalBuffers)
{
	TargetRuntimes.Empty();
	FileType = Type;
	FileData = Buffer;
	AdditionalFileData.Empty();
	for (auto& Element : AdditionalBuffers)
	{
		AdditionalFileData.Emplace(Element.Key, Element.Value);
	}
	RuntimeSettings.Empty();
	FPlatformMisc::CreateGuid(FileId);
	ModelData.Empty();
}

TArrayView<const FString> UNNEModelData::GetTargetRuntimes() const 
{ 
	return TargetRuntimes;
}

void UNNEModelData::SetTargetRuntimes(TArrayView<const FString> RuntimeNames)
{
	TargetRuntimes = RuntimeNames;

	if (RuntimeNames.Num() > 0)
	{
		TArray<FString, TInlineAllocator<10>> CookedRuntimes;
		ModelData.GetKeys(CookedRuntimes);
		for (const FString& Runtime : CookedRuntimes)
		{
			if (!TargetRuntimes.Contains(Runtime))
			{
				ModelData.Remove(Runtime);
			}
		}
		ModelData.Compact();
	}
}

FString UNNEModelData::GetFileType() const
{
	return FileType;
}

TConstArrayView64<uint8> UNNEModelData::GetFileData() const
{
	return FileData;
}

TConstArrayView64<uint8> UNNEModelData::GetAdditionalFileData(const FString& Key) const
{
	return AdditionalFileData.Contains(Key) ? AdditionalFileData[Key] : TConstArrayView<uint8>();
}

void UNNEModelData::ClearFileDataAndFileType()
{
	FileType = "";
	FileData.Empty();
	AdditionalFileData.Empty();
}

UNNEModelData::EDeserializeRuntimeSettings UNNEModelData::DeserializeRuntimeSettings(const TMap<FString, TConstArrayView64<uint8>>& InAdditionalFileData, UObject& InOutSettings)
{
	using namespace UE::NNE::ModelData;
	
	if (const FString* Key = FindLongestKeyWithPrefix(InAdditionalFileData, RuntimeSettingsKeyPrefix))
	{
		ESerializationStatus Status = UE::NNE::ModelData::DeserializeRuntimeSettings(InAdditionalFileData[*Key], &InOutSettings);
		switch (Status)
		{
			case ESerializationStatus::Success:
				return EDeserializeRuntimeSettings::Success;
			case ESerializationStatus::InvalidCustomVersion:
				return EDeserializeRuntimeSettings::InvalidCustomVersion;
			case ESerializationStatus::NewerCustomVersion:
				return EDeserializeRuntimeSettings::NewerCustomVersion;
			case ESerializationStatus::SerializationError:
				return EDeserializeRuntimeSettings::SerializationError;
			default:
				checkNoEntry();
				return EDeserializeRuntimeSettings::SerializationError;
		}
	}
	return EDeserializeRuntimeSettings::Success;
}

UNNEModelData::EGetRuntimeSettingsStatus UNNEModelData::GetRuntimeSettings(const FString& InRuntimeName, UObject*& OutSettings) const
{
	check(!InRuntimeName.IsEmpty());

	using namespace UE::NNE::ModelData;

	TWeakInterfacePtr<INNERuntime> Runtime = UE::NNE::GetRuntime(InRuntimeName);
	if (!Runtime.IsValid())
	{
		return EGetRuntimeSettingsStatus::RuntimeNotFound;
	}
	UObject* DefaultSetting = Runtime.Get()->CreateDefaultRuntimeSettings();
	if (!DefaultSetting)
	{
		return EGetRuntimeSettingsStatus::RuntimeUnsupported;
	}
	if (const TArray64<uint8>* Data = RuntimeSettings.Find(InRuntimeName))
	{
		ESerializationStatus Status = UE::NNE::ModelData::DeserializeRuntimeSettings(*Data, DefaultSetting);
		if (Status != ESerializationStatus::Success)
		{
			switch (Status)
			{
				case ESerializationStatus::InvalidCustomVersion:
					return EGetRuntimeSettingsStatus::InvalidCustomVersion;
				case ESerializationStatus::NewerCustomVersion:
					return EGetRuntimeSettingsStatus::NewerCustomVersion;
				case ESerializationStatus::SerializationError:
					return EGetRuntimeSettingsStatus::SerializationError;
				default:
					checkNoEntry();
					return EGetRuntimeSettingsStatus::SerializationError;
			}
		}
	}
	OutSettings = DefaultSetting;
	return EGetRuntimeSettingsStatus::Success;
}

UNNEModelData::ESetRuntimeSettingsStatus UNNEModelData::SetRuntimeSettings(const FString& InRuntimeName, UObject& InSettings)
{
	check(!InRuntimeName.IsEmpty());

	using namespace UE::NNE::ModelData;

	TWeakInterfacePtr<INNERuntime> Runtime = UE::NNE::GetRuntime(InRuntimeName);
	if (!Runtime.IsValid())
	{
		return ESetRuntimeSettingsStatus::RuntimeNotFound;
	}
	UObject* DefaultSetting = Runtime.Get()->CreateDefaultRuntimeSettings();
	if (!DefaultSetting)
	{
		return ESetRuntimeSettingsStatus::RuntimeUnsupported;
	}
	if (DefaultSetting->GetClass() != InSettings.GetClass())
	{
		return ESetRuntimeSettingsStatus::WrongSettingType;
	}
	TArray64<uint8> SerializedSetting;
	ESerializationStatus Status = SerializeRuntimeSettings(&InSettings, SerializedSetting);
	if(Status != ESerializationStatus::Success)
	{
		check(Status == ESerializationStatus::SerializationError);
		return ESetRuntimeSettingsStatus::SerializationError;
	}
	RuntimeSettings.Add(InRuntimeName, SerializedSetting);
	// Invalidate model data cache
	ModelData.Remove(InRuntimeName);
	return ESetRuntimeSettingsStatus::Success;
}

FGuid UNNEModelData::GetFileId() const
{
	return FileId;
}

TSharedPtr<UE::NNE::FSharedModelData> UNNEModelData::GetModelData(const FString& RuntimeName)
{
	// Check model data is supporting the requested target runtime
	TArrayView<const FString> TargetRuntimesNames = GetTargetRuntimes();
	if (!TargetRuntimesNames.IsEmpty() && !TargetRuntimesNames.Contains(RuntimeName))
	{
		UE_LOGF(LogNNE, Error, "Runtime '%ls' is not among the target runtimes. Target runtimes are: ", *RuntimeName);
		for (const FString& TargetRuntimesName : TargetRuntimesNames)
		{
			UE_LOGF(LogNNE, Error, "- %ls", *TargetRuntimesName);
		}
		return TSharedPtr<UE::NNE::FSharedModelData>();
	}

	// Check if we have a local cache hit
	if (TSharedPtr<UE::NNE::FSharedModelData>* LocalDataPtr = ModelData.Find(RuntimeName))
	{
		return *LocalDataPtr;
	}

	// After this point FileData is required to either get the cache id or recreate it from scratch
	if (FileData.Num() < 1)
	{
		UE_LOGF(LogNNE, Error, "Cannot create model data from empty file data.");
		return TSharedPtr<UE::NNE::FSharedModelData>();
	}

	// Try to create the model
	TSharedPtr<UE::NNE::FSharedModelData> CreatedData = CreateModelData(RuntimeName, nullptr);
	if (!CreatedData.IsValid() || CreatedData->GetView().Num() < 1)
	{
		return TSharedPtr<UE::NNE::FSharedModelData>();
	}

	// Cache the model
	ModelData.Add(RuntimeName, CreatedData);

	return CreatedData;
}

void UNNEModelData::ClearModelData()
{
	ModelData.Empty();
}

TSharedPtr<UE::NNE::FSharedModelData> UNNEModelData::CreateModelData(const FString& RuntimeName, const ITargetPlatform* TargetPlatform) const
{
	TWeakInterfacePtr<INNERuntime> NNERuntime = UE::NNE::GetRuntime<INNERuntime>(RuntimeName);
	if (NNERuntime.IsValid())
	{
		TMap<FString, TConstArrayView64<uint8>> AdditionalFileDataView;
		for (auto& Element : AdditionalFileData)
		{
			AdditionalFileDataView.Emplace(Element.Key, Element.Value);
		}
		if (const TArray64<uint8>* Settings = RuntimeSettings.Find(RuntimeName))
		{
			if (const FString* LongestKey = UE::NNE::ModelData::FindLongestKeyWithPrefix(AdditionalFileDataView, RuntimeSettingsKeyPrefix))
			{
				FString Key;
				Key.Reserve(LongestKey->Len() + 1);
				Key = *LongestKey;
				Key.AppendChar(TEXT('_'));
				AdditionalFileDataView.Add(Key, *Settings);
			}
			else
			{
				AdditionalFileDataView.Add(RuntimeSettingsKeyPrefix, *Settings);
			}
		}
		INNERuntime::ECanCreateModelDataStatus CanCreateModelDataStatus = NNERuntime->CanCreateModelData(FileType, FileData, AdditionalFileDataView, FileId, TargetPlatform);
		if (CanCreateModelDataStatus == INNERuntime::ECanCreateModelDataStatus::Ok)
		{
			return NNERuntime->CreateModelData(FileType, FileData, AdditionalFileDataView, FileId, TargetPlatform);
		}
		else if (CanCreateModelDataStatus == INNERuntime::ECanCreateModelDataStatus::FailFileIdNotSupported)
		{
			UE_LOGF(LogNNE, Display, "Runtime %ls does not support Filetype: %ls, skipping the model data creation for model with id %ls ", *RuntimeName, *FileType, *FileId.ToString(EGuidFormats::Digits).ToLower());
		}
		else
		{
			UE_LOGF(LogNNE, Warning, "Runtime %ls cannot create the model data with id %ls (Filetype: %ls)", *RuntimeName, *FileId.ToString(EGuidFormats::Digits).ToLower(), *FileType);
		}
	}
	else
	{
		UE_LOGF(LogNNE, Error, "No runtime '%ls' found. Valid runtimes are: ", *RuntimeName);
		TArray<FString> Runtimes = UE::NNE::GetAllRuntimeNames();
		for (int32 i = 0; i < Runtimes.Num(); i++)
		{
			UE_LOGF(LogNNE, Error, "- %ls", *Runtimes[i]);
		}
	}
	return TSharedPtr<UE::NNE::FSharedModelData>();
}
