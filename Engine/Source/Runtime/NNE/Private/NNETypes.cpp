// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNETypes.h"
#include "NNE.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NNETypes)

namespace UE::NNE::TypesUtils
{
	template <class T> bool AreShapeEqual(const T& lhs, const T& rhs)
	{
		check(lhs.Rank() <= lhs.MaxRank);

		if (lhs.Rank() != rhs.Rank())
			return false;

		for (int32 i = 0; i < lhs.Rank(); ++i)
		{
			if (lhs.GetData()[i] != rhs.GetData()[i])
				return false;
		}

		return true;
	}
}

namespace UE::NNE
{

bool FSymbolicTensorShape::IsConcrete() const
{
	for (int32 i = 0; i < Rank(); ++i)
	{
		if (Data[i] < 0)
		{
			return false;
		}
	}
	return true;
}

bool FSymbolicTensorShape::operator==(const FSymbolicTensorShape& OtherShape) const
{
	return TypesUtils::AreShapeEqual(*this, OtherShape);
}

bool FSymbolicTensorShape::operator!=(const FSymbolicTensorShape& OtherShape) const { return !(*this == OtherShape); }

void FSymbolicTensorShape::operator=(const FSymbolicTensorShape& OtherShape)
{
	check(OtherShape.Rank() <= MaxRank);
	Data = OtherShape.Data;
}

FSymbolicTensorShape FSymbolicTensorShape::Make(TConstArrayView<int32> Data)
{
	checkf(Data.Num() <= MaxRank, TEXT("Cannot create symbolic tensor shape, input is rank %d while max rank is %d"), Data.Num(), MaxRank);

	FSymbolicTensorShape Shape;
	Shape.Data.Append(Data);
	return Shape;
}

uint64 FTensorShape::Volume() const
{
	uint64 Result = 1;

	for (int32 Idx = 0; Idx < Data.Num(); ++Idx)
	{
		Result *= Data[Idx];
	}

	return Result;
};

bool FTensorShape::IsCompatibleWith(const FSymbolicTensorShape& SymbolicShape) const
{
	if (Rank() != SymbolicShape.Rank())
	{
		return false;
	}
	for (int32 i = 0; i < Rank(); ++i)
	{
		if (SymbolicShape.GetData()[i] >= 0 && SymbolicShape.GetData()[i] != Data[i])
		{
			return false;
		}
	}
	return true;
}

bool FTensorShape::operator==(const FTensorShape& OtherShape) const
{
	return TypesUtils::AreShapeEqual(*this, OtherShape);
}

bool FTensorShape::operator!=(const FTensorShape& OtherShape) const { return !(*this == OtherShape); }

void FTensorShape::operator=(const FTensorShape& OtherShape)
{
	check(OtherShape.Rank() <= MaxRank);
	Data = OtherShape.Data;
}

FTensorShape FTensorShape::Make(TConstArrayView<uint32> Data)
{
	checkf(Data.Num() <= MaxRank, TEXT("Cannot create tensor shape, input is rank %d while max rank is %d"), Data.Num(), MaxRank);

	FTensorShape Shape;
	Shape.Data.Append(Data);
	return Shape;
}

FTensorShape FTensorShape::MakeFromSymbolic(const FSymbolicTensorShape& SymbolicShape)
{
	FTensorShape ConcreteShape;
	for (int32 i = 0; i < SymbolicShape.GetData().Num(); ++i)
	{
		int32 Dim = SymbolicShape.GetData()[i];
		ConcreteShape.Data.Add(Dim < 0 ? 1 : Dim);
	}
	return ConcreteShape;
}

int32 GetTensorDataTypeSizeInBytes(ENNETensorDataType InType)
{
	switch (InType)
	{
	case ENNETensorDataType::Complex128:
		return 16;

	case ENNETensorDataType::Complex64:
		return 8;

	case ENNETensorDataType::Double:
	case ENNETensorDataType::Int64:
	case ENNETensorDataType::UInt64:
		return 8;

	case ENNETensorDataType::Float:
	case ENNETensorDataType::Int32:
	case ENNETensorDataType::UInt32:
		return 4;

	case ENNETensorDataType::Half:
	case ENNETensorDataType::BFloat16:
	case ENNETensorDataType::Int16:
	case ENNETensorDataType::UInt16:
		return 2;

	case ENNETensorDataType::Int8:
	case ENNETensorDataType::UInt8:
	case ENNETensorDataType::Char:
	case ENNETensorDataType::Boolean:
		return 1;
	}

	return 0;
}

FTensorDesc FTensorDesc::Make(const FString& Name, const FSymbolicTensorShape& Shape, ENNETensorDataType DataType)
{
	FTensorDesc Desc;
	Desc.Name = Name;
	Desc.DataType = DataType;
	Desc.Shape = Shape;
	return Desc;
}

} // namespace UE::NNE
