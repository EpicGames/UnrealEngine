// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/RemoveIf.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "NNERuntime.h"
#include "NNEStatus.h"
#include "UObject/WeakInterfacePtr.h"

NNE_API DECLARE_LOG_CATEGORY_EXTERN(LogNNE, Log, All);

namespace UE::NNE
{
	using ERegisterRuntimeStatus = EResultStatus;
	using EUnregisterRuntimeStatus = EResultStatus;

	/**
	 * Register a runtime to make it accessible to NNE clients.
	 *
	 * The caller needs to keep a strong pointer to the runtime object to prevent it from being garbage collected.
	 *
	 * @param Runtime A weak interface pointer to the runtime to be registered.
	 * @return Status indicating success or failure (e.g. if the runtime already has been registered).
	 */
	NNE_API ERegisterRuntimeStatus RegisterRuntime(TWeakInterfacePtr<INNERuntime> Runtime);

	/**
	 * Unregister a registered runtime.
	 *
	 * @param Runtime A weak interface pointer to the runtime to be unregistered.
	 * @return Status indicating success or failure (e.g. if the runtime has not been registered).
	 */
	NNE_API EUnregisterRuntimeStatus UnregisterRuntime(TWeakInterfacePtr<INNERuntime> Runtime);
	
	/**
	 * List and return all registered runtime names.
	 *
	 * @return An array containing runtime names of all registered runtimes.
	 */
	NNE_API TArray<FString> GetAllRuntimeNames();

	/**
	 * Find and return a runtime by name.
	 *
	 * This function tries to find a runtime by name.
	 *
	 * @param Name The name of the runtime.
	 * @return A weak pointer to the runtime if it has been found or an invalid pointer otherwise.
	 */
	NNE_API TWeakInterfacePtr<INNERuntime> GetRuntime(const FString& Name);

	/**
	 * Find and return a runtime by name and interface.
	 *
	 * This function tries to find a runtime by name and casts it to the interface passed as template argument (e.g. INNERuntime, INNERuntimeCPU or INNERuntimeRDG).
	 *
	 * @param Name The name of the runtime.
	 * @return A weak pointer to the runtime if it has been found and implements the interface in the template argument or an invalid pointer otherwise.
	 */
	template<class T>
	TWeakInterfacePtr<T> GetRuntime(const FString& Name)
	{
		TWeakInterfacePtr<INNERuntime> Runtime = GetRuntime(Name);

		T* RuntimePtr = Cast<T>(Runtime.Get());

		return TWeakInterfacePtr<T>(RuntimePtr);
	}

	/**
	 * List and return all registered runtime names that implement the provided interface.
	 *
	 * @return An array containing runtime names of all registered runtimes that implement the interface in the template argument.
	 */
	template<class T>
	TArray<FString> GetAllRuntimeNames()
	{
		TArray<FString> Result = GetAllRuntimeNames();
		Result.SetNum(Algo::RemoveIf(Result, [] (const FString &RuntimeName)
		{
			return !GetRuntime<T>(RuntimeName).IsValid();
		}));

		return Result;
	}
}
