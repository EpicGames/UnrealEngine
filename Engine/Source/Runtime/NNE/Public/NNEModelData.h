// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "Memory/SharedBuffer.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"

#include "NNEModelData.generated.h"

namespace UE::NNE
{
	/**
	 * This class implements a reference counted view on an immutable memory buffer representing model data.
	 * 
	 * It allows runtimes to reference results of GetModelData() even if they outlive UNNEModelData.
	 */
	class FSharedModelData
	{

	public:
		/**
		 * Constructor to shared model data.
		 * 
		 * @param InData The shared buffer containing the model data. InData must be owned and the memory aligned with InMemoryAlignment.
		 * @param InMemoryAlignment The memory alignment with which InData has been aligned. A value <= 1 indicates arbitrary memory alignment.
		 */
		NNE_API FSharedModelData(FSharedBuffer InData, uint32 InMemoryAlignment);

		/**
		 * Constructor to create empty data.
		 */
		NNE_API FSharedModelData();

		/**
		 * Get a const array view on the shared data which is guaranteed to remain valid as long as this objects exists.
		 *
		 * @return A const array view of the shared data.
		 */
		NNE_API TConstArrayView64<uint8> GetView() const;

		/**
		 * Get the memory alignment with which the data has been aligned.
		 *
		 * @return Memory alignment with which the data has been aligned. A value <= 1 indicates arbitrary memory alignment.
		 */
		NNE_API uint32 GetMemoryAlignment() const;

	private:

		/**
		 * The shared buffer containing the model data. Data must be aligned with MemoryAlignment.
		 */
		FSharedBuffer Data;
		
		/**
		 * The memory alignment with which Data has been aligned. A value <= 1 indicates arbitrary memory alignment.
		 */
		uint32 MemoryAlignment;
	};
}

/**
 * This class represents assets that store neural network model data.
 *
 * Neural network models typically consist of a graph of operations and corresponding parameters as e.g. weights.
 * UNNEModelData assets store such model data as imported e.g. by the UNNEModelDataFactory class.
 * An INNERuntime object retrieved by UE::NNE::GetRuntime<T>(const FString& Name) can be used to create an inferable neural network model.
 */
UCLASS(BlueprintType, Category = "NNE", MinimalAPI)
class UNNEModelData : public UObject
{
	GENERATED_BODY()

public:

	enum class EDeserializeRuntimeSettings : uint8
	{
		Success = 0,
		InvalidCustomVersion = 1,
		NewerCustomVersion = 2,
		SerializationError = 3,
	};

	enum class EGetRuntimeSettingsStatus : uint8
	{
		Success = 0,
		RuntimeNotFound = 1,
		RuntimeUnsupported = 2,
		InvalidCustomVersion = 3,
		NewerCustomVersion = 4,
		SerializationError = 5,
	};

	enum class ESetRuntimeSettingsStatus : uint8
	{
		Success = 0,
		RuntimeNotFound = 1,
		RuntimeUnsupported = 2,
		WrongSettingType = 3,
		SerializationError = 4,
	};

	// UObject interface
	NNE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	NNE_API virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface

	/**
	 * Initialize the model data with a copy of the data inside Buffer.
	 *
	 * This function is called by the UNNEModelDataFactory class when importing a neural network model file.
	 *
	 * @param Type A string identifying the type of data inside this asset. Corresponds to the extension of the imported file.
	 * @param Buffer The raw binary file data of the imported model to be copied into this asset.
	 * @param AdditionalBuffers Additional raw binary data of the model to be copied into this asset.
	 */
	NNE_API void Init(const FString& Type, TConstArrayView64<uint8> Buffer, const TMap<FString, TConstArrayView64<uint8>>& AdditionalBuffers = TMap<FString, TConstArrayView64<uint8>>());

	/**
	 * In editor: Get the target runtimes this model data will be cooked for. An empty list means all runtimes.
	 * In standalone: An empty list.
	 *
	 * @return The target runtimes names.
	 */
	NNE_API TArrayView<const FString> GetTargetRuntimes() const;

	/**
	 * Set the target runtimes this model data will be cooked for. An empty list means all runtimes.
	 *
	 * @param RuntimeNames The target runtimes names.
	 */
	NNE_API void SetTargetRuntimes(TArrayView<const FString> RuntimeNames);

	/**
	 * Get the type of data inside FileData.
	 *
	 * In editor: The FileType identifies the type of data inside FileData and typically is the extension of the file used to create the asset.
	 * In standalone: An empty string.
	 *
	 * @return The FileType.
	 */
	NNE_API FString GetFileType() const;

	/**
	 * Get read only access to FileData.
	 *
	 * In editor: The FileData contains the binary data of the file which has been used to create the asset.
	 * In standalone: An empty array.
	 *
	 * @return The FileData.
	 */
	NNE_API TConstArrayView64<uint8> GetFileData() const;

	/**
	 * Get read only access to AdditionalFileData.
	 *
	 * In editor: The AdditionalFileData contains the additional binary data of the neural network model.
	 * In standalone: An empty map.
	 *
	 * @return The AdditionalFileData with a given Key if it exists and an empty view in standalone or when the key does not exist.
	 */
	NNE_API TConstArrayView64<uint8> GetAdditionalFileData(const FString& Key) const;

	/**
	 * Clears the FileData and the FileType.
	 *
	 * Caution, if the FileData is cleared, no more models can be created on runtimes that do not already have ModelData inside this asset.
	 */
	NNE_API void ClearFileDataAndFileType();


	/**
	 * Uses AdditionalFileData that is passed into CanCreateModelData and CreateModelData to populate InOutSettings
	 *
	 * @param InAdditionalFileData The AdditionalFileData argument that was passed into INNERuntime::CanCreateModelData or INNERuntime::CreateModelData
	 * @param InOutSettings An instance of the RuntimeSettings that is returned from INNERuntime::CreateDefaultRuntimeSettings. If the user hasn't changed
	 * the runtime settings this will be unchanged. Else InOutSettings will be deserialized. On failure the value of InOutSettings is undefined.
	 *
	 * @return Status of deserializing the settings.
	 */
	NNE_API static EDeserializeRuntimeSettings DeserializeRuntimeSettings(const TMap<FString, TConstArrayView64<uint8>>& InAdditionalFileData, UObject& InOutSettings);

	/**
	 * Gets the setting of the given runtime.
	 *
	 * @param InRuntimeName The name of the runtime whose setting to get.
	 * @param OutSettings On success points to the created settings. On Failure OutSettings is left unchanged.
	 *
	 * @return Status of getting the setting.
	 */
	NNE_API EGetRuntimeSettingsStatus GetRuntimeSettings(const FString& InRuntimeName, UObject*& OutSettings) const;

	/**
	 * Sets the setting of the given runtime.
	 *
	 * @param InRuntimeName The name of the runtime whose setting to set.
	 * @param InSettings The setting to set. This needs to have the same type as gets returned by the runtime through INNERuntime::CreateDefaultRuntimeSettings().
	 *
	 * @return Status of setting the setting.
	 */
	NNE_API ESetRuntimeSettingsStatus SetRuntimeSettings(const FString& InRuntimeName, UObject& InSettings);

	/**
	 * Get the FGuid identifying the FileData.
	 *
	 * The FileId is created on import of an asset. It can be used to identify the FileData, e.g. when putting corresponding data into the DDC or caching data locally.
	 * In standalone: An empty FGuid.
	 *
	 * @return The FileId.
	 */
	NNE_API FGuid GetFileId() const;

	/**
	 * Get the cached (editor) or cooked (game) optimized model data for a given runtime.
	 *
	 * This function is used by runtimes when creating a model. In editor, the function will create the optimized model data with the passed runtime in case it has not been cached in the DCC yet. In game, the cooked data is accessed. The returned model data is aligned in memory as requested by the runtime.
	 *
	 * @param RuntimeName The name of the runtime for which the data should be returned.
	 * @return The optimized and runtime specific model data or an invalid TSharedPtr in case of failure.
	 */
	NNE_API TSharedPtr<UE::NNE::FSharedModelData> GetModelData(const FString& RuntimeName);

	/**
	 * Clears the ModelData.
	 *
	 * Caution, if the ModelData is cleared, only runtimes that support cooking on the current platform can create new models from this asset.
	 */
	NNE_API void ClearModelData();

private:
	/**
	 * A list of string of the supported runtime, empty to support them all.
	 */
	TArray<FString> TargetRuntimes;

	/**
	 * A string identifying the type of data inside this asset. Corresponds to the extension of the imported file.
	 */
	FString FileType;

	/**
	 * The raw binary file data of the imported model.
	 */
	TArray64<uint8> FileData;

	/**
	 * Additional raw binary data of the imported model.
	 */
	TMap<FString, TArray64<uint8>> AdditionalFileData;

	/**
	 * Mapping between a runtime name and the serialized version of it's runtime settings.
	 */
	TMap<FString, TArray64<uint8>> RuntimeSettings;

	/**
	 * A Guid that uniquely identifies this model. This is used to cache optimized models in the editor.
	 */
	FGuid FileId;

	/**
	 * The processed / optimized model data for the different runtimes.
	 */
	TMap<FString, TSharedPtr<UE::NNE::FSharedModelData>> ModelData;

	/**
	* Helper method for creating shared model data
	*/
	TSharedPtr<UE::NNE::FSharedModelData> CreateModelData(const FString& RuntimeName, const ITargetPlatform* TargetPlatform) const;

	/**
	* Prefix of the key that stores serialized runtime settings in the version of AdditionalFileData that is passed into INNERuntime::CanCreateModelData and INNERuntime::CreateModelData
	*/
	static inline const FString RuntimeSettingsKeyPrefix = TEXT("RuntimeSettings");
};
