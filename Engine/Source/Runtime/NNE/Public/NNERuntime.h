// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "NNEStatus.h"
#include "UObject/Interface.h"

#include "NNERuntime.generated.h"

class ITargetPlatform;

namespace UE::NNE
{
	class FSharedModelData;
}

UINTERFACE(MinimalAPI)
class UNNERuntime : public UInterface
{
	GENERATED_BODY()
};

/**
 * The base interface of a neural network runtime.
 *
 * This interface is mainly used internally by UNNEModelData to cook a model given in a file format (e.g. .onnx) into runtime specific model data.
 * The model data is then stored inside UNNEModelData which then can be used in game to create an inferable model.
 * See INNERuntimeCPU, INNERuntimeGPU and INNERuntimeRDG on how to create a model given a UNNEModelData asset.
 */
class INNERuntime
{
	GENERATED_BODY()

public:

	enum class ECanCreateModelDataStatus : uint8
	{
		Ok = 0,
		Fail = 1,
		FailFileIdNotSupported = 2
	};

	/**
	 * Get the name of the runtime.
	 *
	 * @return The name of the runtime.
	 */
	virtual FString GetRuntimeName() const = 0;

	/**
	 * Check if the runtime is able to create model data given some file data representing a neural network.
	 *
	 * @param FileType The type of file inside FileData. Corresponds to the file extension (e.g. 'onnx').
	 * @param FileData The raw binary file of a neural network model.
	 * @param AdditionalFileData Additional raw binary data of the neural network model. Can be passed to UNNEModelData::DeserializeRuntimeSettings.
	 * @param FileId The unique identifier representing FileData.
	 * @param TargetPlatform The Interface identifying the target platform for which the data needs to be created. A null pointer indicates the currently compiled/running platform.
	 * @return Ok if the runtime is able to create model data, FailFileIdNotSupported if this fileId is not supported and Fail otherwise.
	 */
	virtual ECanCreateModelDataStatus CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const = 0;

	/**
	 * Create model data given some raw file data.
	 *
	 * @param FileType The type of file inside FileData. Corresponds to the file extension (e.g. 'onnx').
	 * @param FileData The raw binary file of a neural network model.
	 * @param AdditionalFileData Additional raw binary data of the neural network model. Can be passed to UNNEModelData::DeserializeRuntimeSettings.
	 * @param FileId The unique identifier representing FileData.
	 * @param TargetPlatform The Interface identifying the target platform for which the data needs to be created. A null pointer indicates the currently compiled/running platform.
	 * @return A pointer to data representing the runtime specific representation of the model to be stored by UNNEModelData on success or an invalid pointer otherwise.
	 */
	virtual TSharedPtr<UE::NNE::FSharedModelData> CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) = 0;

	/**
	 * Get an id uniquely identifying the model data. This is used by UNNEModelData to create a FCacheKey to get and add the model data from and to a DDC.
	 *
	 * @param FileType The type of file inside FileData. Corresponds to the file extension (e.g. 'onnx').
	 * @param FileData The raw binary file of a neural network model.
	 * @param AdditionalFileData Additional raw binary data of the neural network model.
	 * @param FileId The unique identifier representing FileData.
	 * @param TargetPlatform The Interface identifying the target platform for which the data is produced or by which it will be consumed. A null pointer indicates the currently compiled/running platform.
	 * @return A string that identifies the model data uniquely, e.g. by containing the FileId, model data version and target platform display name.
	 */
	virtual FString GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const = 0;

	/**
	 * Create an instance of the default runtime settings. The Serialize(FArchive& Ar) and GetClass() methods are used to store and display these settings.
	 * 
	 * @return A new instance of the runtime settings or nullptr if runtime settings are not supported by this runtime.
	 */
	virtual UObject* CreateDefaultRuntimeSettings() const
	{
		return nullptr;
	}
};