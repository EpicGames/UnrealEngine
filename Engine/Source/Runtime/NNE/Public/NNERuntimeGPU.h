// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNERuntimeRunSync.h"
#include "NNEStatus.h"
#include "UObject/Interface.h"

#include "NNERuntimeGPU.generated.h"

class UNNEModelData;

namespace UE::NNE
{
UE_DEPRECATED(5.5, "Use UE::NNE::FTensorBindingCPU instead.")
typedef FTensorBindingCPU FTensorBindingGPU;

/**
 * The interface of a model instance that can run on GPU.
 *
 * Use UE::NNE::IModelGPU::CreateModelInstance() to get a model instance.
 * Use UE::NNE::GetRuntime<INNERuntimeGPU>(RuntimeName) to get a runtime capable of creating GPU models.
 */
class IModelInstanceGPU : public IModelInstanceRunSync
{
};

/**
 * The interface of a model capable of creating model instance that can run on GPU.
 *
 * Use UE::NNE::GetRuntime<INNERuntimeGPU>(RuntimeName) to get a runtime capable of creating GPU models.
 */
class IModelGPU
{
public:

	virtual ~IModelGPU() = default;

	/**
	 * Create a model instance for inference
	 *
	 * The runtime have the opportunity to share the model weights among multiple IModelInstanceGPU created from an IModelGPU instance, however this is not mandatory.
	 * The caller can decide to convert the result into a shared pointer if required (e.g. if the model needs to be shared with an async task for evaluation).
	 *
	 * @return A caller owned model representing the neural network instance created.
	 */
	virtual TSharedPtr<UE::NNE::IModelInstanceGPU> CreateModelInstanceGPU() = 0;
};

} // UE::NNE

UINTERFACE(MinimalAPI)
class UNNERuntimeGPU : public UInterface
{
	GENERATED_BODY()
};

/**
 * The interface of a neural network runtime capable of creating GPU models.
 *
 * Call UE::NNE::GetRuntime<INNERuntimeGPU>(RuntimeName) to get a runtime implementing this interface.
 */
class INNERuntimeGPU
{
	GENERATED_BODY()
	
public:

	using ECanCreateModelGPUStatus = UE::NNE::EResultStatus;

	/**
	 * Check if the runtime is able to create a model given some ModelData.
	 *
	 * @param ModelData The model data for which to create a model.
	 * @return True if the runtime is able to create the model, false otherwise.
	 */
	virtual ECanCreateModelGPUStatus CanCreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) const = 0;
	
	/**
	 * Create a model given some ModelData.
	 *
	 * The caller must make sure ModelData remains valid throughout the call.
	 * ModelData is not required anymore after the model has been created.
	 * The caller can decide to convert the result into a shared pointer if required (e.g. if the model needs to be shared with an async task for evaluation).
	 *
	 * @param ModelData The model data for which to create a model.
	 * @return A caller owned model representing the neural network created from ModelData.
	 */
	virtual TSharedPtr<UE::NNE::IModelGPU> CreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) = 0;
};