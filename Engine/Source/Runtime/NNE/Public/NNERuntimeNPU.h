// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNERuntimeRunSync.h"
#include "NNEStatus.h"
#include "UObject/Interface.h"

#include "NNERuntimeNPU.generated.h"

class UNNEModelData;

namespace UE::NNE
{

/**
 * The interface of a model instance that can run on NPU.
 *
 * Use UE::NNE::IModelNPU::CreateModelInstance() to get a model instance.
 * Use UE::NNE::GetRuntime<INNERuntimeNPU>(RuntimeName) to get a runtime capable of creating NPU models.
 */
class IModelInstanceNPU : public IModelInstanceRunSync
{
};

/**
 * The interface of a model capable of creating model instance that can run on NPU.
 *
 * Use UE::NNE::GetRuntime<INNERuntimeNPU>(RuntimeName) to get a runtime capable of creating NPU models.
 */
class IModelNPU
{
public:

	virtual ~IModelNPU() = default;

	/**
	 * Create a model instance for inference
	 *
	 * The runtime have the opportunity to share the model weights among multiple IModelInstanceNPU created from an IModelNPU instance, however this is not mandatory.
	 * The caller can decide to convert the result into a shared pointer if required (e.g. if the model needs to be shared with an async task for evaluation).
	 *
	 * @return A caller owned model representing the neural network instance created.
	 */
	virtual TSharedPtr<UE::NNE::IModelInstanceNPU> CreateModelInstanceNPU() = 0;
};

} // UE::NNE

UINTERFACE(MinimalAPI)
class UNNERuntimeNPU : public UInterface
{
	GENERATED_BODY()
};

/**
 * The interface of a neural network runtime capable of creating NPU models.
 *
 * Call UE::NNE::GetRuntime<INNERuntimeNPU>(RuntimeName) to get a runtime implementing this interface.
 */
class INNERuntimeNPU
{
	GENERATED_BODY()
	
public:

	using ECanCreateModelNPUStatus = UE::NNE::EResultStatus;

	/**
	 * Check if the runtime is able to create a model given some ModelData.
	 *
	 * @param ModelData The model data for which to create a model.
	 * @return True if the runtime is able to create the model, false otherwise.
	 */
	virtual ECanCreateModelNPUStatus CanCreateModelNPU(const TObjectPtr<UNNEModelData> ModelData) const = 0;
	
	/**
	 * Create a model given some ModelData.
	 *
	 * The caller must make sure ModelData remains valid throughout the call.
	 * ModelData is not required anymore after the model has been created.
	 * The caller can decide to convert the result into a shared pointer if required (e.g. if the model needs to be shared with an async task for evaluation).
	 *
	 * @param ModelData The model data for which to create a model.
	 * @return A caller owned model representing the neural network created from ModelData.
	 */
	virtual TSharedPtr<UE::NNE::IModelNPU> CreateModelNPU(const TObjectPtr<UNNEModelData> ModelData) = 0;
};