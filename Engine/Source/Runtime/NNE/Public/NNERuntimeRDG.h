// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "NNEStatus.h"
#include "NNETypes.h"
#include "UObject/Interface.h"

#include "NNERuntimeRDG.generated.h"

class FRDGBuffer;
using FRDGBufferRef = FRDGBuffer*;

class FRDGBuilder;

class UNNEModelData;

namespace UE::NNE
{

/**
 * The tensor binding for passing input and output to IModelRDG.
 *
 * Memory is owned by the caller. The caller must make sure the buffer is large enough.
 */
struct FTensorBindingRDG
{
	FRDGBufferRef Buffer;
};

/**
 * The interface of a model instance that can run on RDG.
 *
 * Use UE::NNE::IModelRDG::CreateModelInstance() to get a model instance.
 * Use UE::NNE::GetRuntime<INNERuntimeRDG>(RuntimeName) to get a runtime capable of creating RDG models.
 */
class IModelInstanceRDG
{
public:

	using ESetInputTensorShapesStatus = EResultStatus;
	using EEnqueueRDGStatus = EResultStatus;

	virtual ~IModelInstanceRDG() = default;

	/**
	 * Get the input tensor descriptions as defined by the model, potentially with variable dimensions.
	 *
	 * @return An array containing a tensor descriptor for each input tensor of the model.
	 */
	virtual TConstArrayView<FTensorDesc> GetInputTensorDescs() const = 0;

	/**
	 * Get the output tensor descriptions as defined by the model, potentially with variable dimensions.
	 *
	 * @return An array containing a tensor descriptor for each output tensor of the model.
	 */
	virtual TConstArrayView<FTensorDesc> GetOutputTensorDescs() const = 0;

	/**
	 * Get the input shapes.
	 *
	 * SetInputTensorShapes must be called prior of running a model.
	 *
	 * @return An array of input shapes or an empty array if SetInputTensorShapes has not been called.
	 */
	virtual TConstArrayView<FTensorShape> GetInputTensorShapes() const = 0;

	/**
	 * Getters for outputs shapes if they were already resolved.
	 *
	 * Output shapes might be resolved after a call to SetInputTensorShapes if the model and runtime supports it.
	 * Otherwise they will be resolved while running the model
	 *
	 * @return An array of output shapes or an empty array if not resolved yet.
	 */
	virtual TConstArrayView<FTensorShape> GetOutputTensorShapes() const = 0;
	
	/**
	 * Prepare the model to be run with the given input shape.
	 * 
	 * The call is mandatory before a model can be run.
	 * This function might be called from the render thread, if not it is up to the caller to ensure thread safety.
	 * The function will run shape inference and resolve, if possible, the output shapes which can then be accessed by calling GetOutputTensorShapes().
	 * This is a potentially expensive call and should be called lazily if possible.
	 *
	 * @param InInputShapes The input shapes to prepare the model with.
	 * @return Status indicating success or failure.
	 */
	virtual ESetInputTensorShapesStatus SetInputTensorShapes(TConstArrayView<FTensorShape> InInputShapes) = 0;

	/**
	 * Enqueue the model graph to a FRDGBuilder.
	 *
	 * This function must be called from the render thread.
	 * SetInputTensorShapes must be called prior to this call.
	 * The caller owns the memory inside the bindings and must make sure that they are big enough.
	 *
	 * @param RDGBuilder The RDG builder to which the neural network operations are enqueued.
	 * @param InInputTensors An array containing tensor bindings for each input tensor with caller owned memory containing the input data.
	 * @param InOutputTensors An array containing tensor bindings for each output tensor with caller owned memory big enough to contain the results on success.
	 * @return Status indicating success or failure.
	 */
	virtual EEnqueueRDGStatus EnqueueRDG(FRDGBuilder& RDGBuilder, TConstArrayView<FTensorBindingRDG> Inputs, TConstArrayView<FTensorBindingRDG> Outputs) = 0;
};

/**
 * The interface of a model capable of creating model instance that can run on RDG.
 *
 * Use UE::NNE::GetRuntime<INNERuntimeRDG>(RuntimeName) to get a runtime capable of creating RDG models.
 */
class IModelRDG
{
public:

	virtual ~IModelRDG() = default;

	/**
	 * Create a model instance for inference
	 *
	 * The runtime have the opportunity to share the model weights among multiple IModelInstanceRDG created from an IModelRDG instance, however this is not mandatory.
	 * The caller can decide to convert the result into a shared pointer if required (e.g. if the model needs to be shared with an async task for evaluation).
	 *
	 * @return A caller owned model representing the neural network instance created.
	 */
	virtual TSharedPtr<UE::NNE::IModelInstanceRDG> CreateModelInstanceRDG() = 0;
};

} // UE::NNE

UINTERFACE(MinimalAPI)
class UNNERuntimeRDG : public UInterface
{
	GENERATED_BODY()
};

/**
 * The interface of a neural network runtime capable of creating RDG models.
 *
 * Call UE::NNE::GetRuntime<INNERuntimeRDG>(RuntimeName) to get a runtime implementing this interface.
 */
class INNERuntimeRDG
{
	GENERATED_BODY()

public:

	using ECanCreateModelRDGStatus = UE::NNE::EResultStatus;

	/**
	 * Check if the runtime is able to create a model given some ModelData.
	 *
	 * @param ModelData The model data for which to create a model.
	 * @return True if the runtime is able to create the model, false otherwise.
	 */
	virtual ECanCreateModelRDGStatus CanCreateModelRDG(const TObjectPtr<UNNEModelData> ModelData) const = 0;
	
	/**
	 * Create a model given some ModelData.
	 *
	 * The caller must make sure ModelData remains valid throughout the call.
	 * ModelData is not required anymore after the model has been created.
	 * The caller can decide to convert the result into a shared pointer if required (e.g. if the model needs to be shared with the render thread).
	 *
	 * @param ModelData The model data for which to create a model.
	 * @return A caller owned model representing the neural network created from ModelData.
	 */
	virtual TSharedPtr<UE::NNE::IModelRDG> CreateModelRDG(const TObjectPtr<UNNEModelData> ModelData) = 0;
};