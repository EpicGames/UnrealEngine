// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "NNEStatus.h"
#include "NNETypes.h"


namespace UE::NNE
{

/**
 * The tensor binding for passing input and output to from CPU memory.
 *
 * Memory is owned by the caller. The caller must make sure the buffer is large enough and at least as large as SizeInBytes.
 */
struct FTensorBindingCPU
{
	void*	Data;
	uint64	SizeInBytes;
};

/**
 * The interface of a model instance that can run synchronously from CPU memory.
 *
 * Use UE::NNE::IModelXXX::CreateModelInstance() to get a model instance.
 * Use UE::NNE::GetRuntime<INNERuntimeXXX>(RuntimeName) to get a runtime capable of creating models.
 */
class IModelInstanceRunSync
{
public:

	using ESetInputTensorShapesStatus = EResultStatus;
	using ERunSyncStatus = EResultStatus;

	virtual ~IModelInstanceRunSync() = default;

	/**
	 * Get the input tensor descriptions as defined by the model, potentially with variable dimensions.
	 *
	 * @return An array containing a tensor descriptor for each input tensor of the model.
	 */
	virtual TConstArrayView<FTensorDesc> GetInputTensorDescs() const = 0;
	
	/**
	 * Get the output tensor descriptions as defined by the model, potentially with variable dimensions.
	 *
	 * @return An array containing a tensor descriptor for each output tensor of the model.
	 */
	virtual TConstArrayView<FTensorDesc> GetOutputTensorDescs() const = 0;

	/**
	 * Get the input shapes.
	 *
	 * SetInputTensorShapes must be called prior of running a model.
	 *
	 * @return An array of input shapes or an empty array if SetInputTensorShapes has not been called.
	 */
	virtual TConstArrayView<FTensorShape> GetInputTensorShapes() const = 0;

	/**
	 * Getters for outputs shapes if they were already resolved.
	 *
	 * Output shapes might be resolved after a call to SetInputTensorShapes if the model and runtime supports it.
	 * Otherwise they will be resolved while running the model
	 *
	 * @return An array of output shapes or an empty array if not resolved yet.
	 */
	virtual TConstArrayView<FTensorShape> GetOutputTensorShapes() const = 0;
	
	/**
	 * Prepare the model to be run with the given input shape.
	 *
	 * The call is mandatory before a model can be run.
	 * The function will run shape inference and resolve, if possible, the output shapes which can then be accessed by calling GetOutputTensorShapes().
	 * This is a potentially expensive call and should be called lazily if possible.
	 *
	 * @param InInputShapes The input shapes to prepare the model with.
	 * @return Status indicating success or failure.
	 */
	virtual ESetInputTensorShapesStatus SetInputTensorShapes(TConstArrayView<FTensorShape> InInputShapes) = 0;

	/**
	 * Evaluate the model synchronously.
	 *
	 * SetInputTensorShapes must be called prior to this call.
	 * This function will block the calling thread until the inference is complete.
	 * The caller owns the memory inside the bindings and must make sure that they are big enough.
	 * Clients can call this function from an async task but must make sure the memory remains valid throughout the evaluation.
	 *
	 * @param InInputTensors An array containing tensor bindings for each input tensor with caller owned memory containing the input data.
	 * @param InOutputTensors An array containing tensor bindings for each output tensor with caller owned memory big enough to contain the results on success.
	 * @return Status indicating success or failure.
	 */
	virtual ERunSyncStatus RunSync(TConstArrayView<FTensorBindingCPU> InInputTensors, TConstArrayView<FTensorBindingCPU> InOutputTensors) = 0;
};

} // UE::NNE