// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::NNE
{

enum class EResultStatus : uint8
{
	Ok = 0,
	Fail = 1
};

} // namespace UE::NNE
