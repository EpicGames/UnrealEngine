// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"

#include "NNETypes.generated.h"

/**
 * The enum lists all tensor data types used in NNE.
 *
 * See UE::NNE::GetTensorDataTypeSizeInBytes(ENNETensorDataType InType) to get the size of each data type in bytes.
 */
UENUM()
enum class ENNETensorDataType : uint8
{
	None,								//!< For empty tensors
	Char,								//!< Character type
	Boolean,							//!< Boolean type
	Half,								//!< 16-bit floating number
	Float,								//!< 32-bit floating number
	Double,								//!< 64-bit floating number
	Int8,								//!< 8-bit signed integer
	Int16,								//!< 16-bit signed integer
	Int32,								//!< 32-bit signed integer
	Int64,								//!< 64-bit signed integer
	UInt8,								//!< 8-bit unsigned integer
	UInt16,								//!< 16-bit unsigned integer
	UInt32,								//!< 32-bit unsigned integer
	UInt64,								//!< 64-bit unsigned integer
	Complex64,							//!< 64-bit Complex Number
	Complex128,							//!< 128-bit Complex Number
	BFloat16							//!< 16-bit floating number
};

namespace UE::NNE
{
	/**
	 * Return the data size in bytes for a tensor data type.
	 *
	 * @param InType the type of the element to consider.
	 * @return the data size in bytes of an element.
	 */
	int32 NNE_API GetTensorDataTypeSizeInBytes(ENNETensorDataType InType);

	/**
	 * A symbolic tensor shape represents the shape of a tensor with potentially variable dimension.
	 * 
	 * The variable dimensions are represented by -1 values. 
	 */
	class FSymbolicTensorShape
	{
	public:
		/**
		 * The maximum number of dimensions supported by the symbolic tensor shape, aka the rank.
		 */
		constexpr static int32	MaxRank = 8;

	private:
		TArray<int32, TInlineAllocator<MaxRank>> Data;

	public:
		/**
		 * Default construct a symbolic tensor shape.
		 */
		FSymbolicTensorShape() = default;

		/**
		 * Construct this symbolic tensor shape from another one.
		 * 
		 * @param OtherShape the other symbolic tensor shape to copy from.
		 */
		FSymbolicTensorShape(const FSymbolicTensorShape& OtherShape) = default;

		/**
		 * Construct a symbolic tensor shape with the given dimensions.
		 *
		 * @param Data an array of dimensions.
		 * @return a symbolic tensor shape with the given dimensions.
		 */
		static NNE_API FSymbolicTensorShape Make(TConstArrayView<int32> Data);

		/**
		 * Get the dimensions of the symbolic tensor shape.
		 *
		 * @return a view on the dimensions of the symbolic tensor shape.
		 */
		inline TConstArrayView<int32> GetData() const { return Data; };
		
		/**
		 * Get the number of dimensions of the symbolic tensor shape, aka the rank.
		 *
		 * @return the number of dimensions of the symbolic tensor shape.
		 */
		inline int32 Rank() const { return Data.Num(); }

		/**
		 * Check if the symbolic tensor shape has no variable dimensions, aka is concrete.
		 *
		 * @return true if the symbolic tensor shape is concrete, false otherwise.
		 */
		NNE_API bool IsConcrete() const;

		/**
		 * Check if the symbolic tensor shape is equal to another one.
		 *
		 * @param OtherShape the other symbolic tensor shape.
		 * @return true if the symbolic tensor shape is equal to the other one, false otherwise.
		 */
		NNE_API bool operator==(const FSymbolicTensorShape& OtherShape) const;
		
		/**
		 * Check if the symbolic tensor shape is different from another one.
		 * 
		 * @param OtherShape the other symbolic tensor shape.
		 * @return true if the symbolic tensor shape is different from the other one, false otherwise.
		 */
		NNE_API bool operator!=(const FSymbolicTensorShape& OtherShape) const;

		/**
		 * Assign this symbolic tensor shape from another one.
		 * 
		 * @param OtherShape the other symbolic tensor shape to copy from.
		 */
		NNE_API void operator=(const FSymbolicTensorShape& OtherShape);
	};

	/**
	 * The concrete shape of a tensor.
	 *
	 * Concrete tensor shapes are well defined through strictly positive values and thus have a defined volume.
	 */
	class FTensorShape
	{
	public:
		/**
		 * The maximum number of dimensions supported by the tensor shape, aka the rank.
		 */
		constexpr static int32	MaxRank = FSymbolicTensorShape::MaxRank;

	private:
		TArray<uint32, TInlineAllocator<MaxRank>> Data;

	public:
		/**
		 * Default construct a tensor shape.
		 */
		FTensorShape() = default;

		/**
		 * Construct this tensor shape from another one.
		 * @param OtherShape the other tensor shape to copy from.
		 */
		FTensorShape(const FTensorShape& OtherShape) = default;

		/**
		 * Construct a tensor shape with the given dimensions.
		 *
		 * @param Data an array of dimensions.
		 * @return a tensor shape with the given dimensions.
		 */
		static NNE_API FTensorShape Make(TConstArrayView<uint32> Data);
		
		/**
		 * Construct a tensor shape from a symbolic tensor shape.
		 * 
		 * Negative dimension values are replaced by 1.
		 *
		 * @param SymbolicShape a symbolic tensor shape.
		 * @return a tensor shape with the given dimensions.
		 */
		static NNE_API FTensorShape MakeFromSymbolic(const FSymbolicTensorShape& SymbolicShape);

		/**
		 * Get the dimensions of the tensor shape.
		 *
		 * @return a view on the dimensions of the tensor shape.
		 */
		inline TConstArrayView<uint32> GetData() const { return Data; };
		
		/**
		 * Get the number of dimensions of the tensor shape, aka the rank.
		 *
		 * @return the number of dimensions of the tensor shape.
		 */
		inline int32 Rank() const { return Data.Num(); }
		
		/**
		 * Get the number of elements in this tensor shape.
		 *
		 * @return the number of element in this tensor shape.
		 */
		NNE_API uint64 Volume() const;
		
		/**
		 * Check if this tensor shape is part of the ensemble of shapes defined by the symbolic shape.
		 * 
		 * For example [1,2] is compatible with [-1,2] and [1,2] but not with [2,2] or [1,2,3].
		 *
		 * @return true if this tensor shape is compatible with the symbolic shape, false otherwise.
		 */
		NNE_API bool IsCompatibleWith(const FSymbolicTensorShape& SymbolicShape) const;

		/**
		 * Check if the tensor shape is equal to another one.
		 * 
		 * @param OtherShape the other tensor shape.
		 * @return true if the tensor shape is equal to the other one, false otherwise.
		 */
		NNE_API bool operator==(const FTensorShape& OtherShape) const;
		
		/**
		 * Check if the tensor shape is different from another one.
		 * 
		 * @param OtherShape the other tensor shape.
		 * @return true if the tensor shape is different from the other one, false otherwise.
		 */
		NNE_API bool operator!=(const FTensorShape& OtherShape) const;

		/**
		 * Assign this tensor shape from another one.
		 * @param OtherShape the other tensor shape to copy from.
		 */
		NNE_API void operator=(const FTensorShape& OtherShape);
	};

	/**
	 * The descriptor for a tensor as model inputs and output.
	 *
	 * A tensor is described by its name, the type of data it contains and it's shape.
	 * Since input and output tensors of a neural network can have dynamic shapes, Shape is symbolic.
	 */
	class FTensorDesc
	{
	private:
		FString					Name;
		ENNETensorDataType		DataType;
		FSymbolicTensorShape	Shape;

		FTensorDesc() = default;

	public:
		/**
		 * Construct a tensor description for model input/output
		 *
		 * @param Name the name of the tensor.
		 * @param Shape the symbolic shape of the tensor.
		 * @param DataType the type of data the tensor contains.
		 * @return a tensor shape with the given dimensions.
		 */
		static NNE_API FTensorDesc Make(const FString& Name, const FSymbolicTensorShape& Shape, ENNETensorDataType DataType);

		/**
		 * Get the name of the tensor
		 * 
		 * @return the name of the tensor.
		 */
		inline const FString& GetName() const { return Name; }
		
		/**
		 * Get the data type of the tensor
		 * 
		 * @return the data type of the tensor.
		 */
		inline ENNETensorDataType GetDataType() const { return DataType; }
		
		/**
		 * Get the size in bytes of one element of the tensor.
		 * 
		 * @return the size in bytes of one element of the tensor.
		 */
		inline uint32 GetElementByteSize() const { return GetTensorDataTypeSizeInBytes(DataType); }

		/**
		 * Get the symbolic shape of the tensor.
		 * 
		 * @return the symbolic shape of the tensor.
		 */
		inline const FSymbolicTensorShape& GetShape() const { return Shape; }
	};

} // namespace UE::NNE
